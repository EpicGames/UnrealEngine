// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlCheckInPromptModule.h"
#include "SourceControlCheckInPrompter.h"
#include "SourceControlWindows.h"
#include "Dialogs/Dialogs.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "ISourceControlModule.h"

#define LOCTEXT_NAMESPACE "SourceControlCheckInPromptModule"

TAutoConsoleVariable<bool> CVarSourceControlEnablePeriodicCheckInPrompt(
	TEXT("SourceControl.CheckInPrompt.EnablePeriodic"),
	false,
	TEXT("Enables a periodic check-in prompt, reminding the user to check-in to avoid losing work."),
	ECVF_Default);

TAutoConsoleVariable<bool> CVarSourceControlEnableOnPublishCheckInPrompt(
	TEXT("SourceControl.CheckInPrompt.EnableOnPublish"),
	false,
	TEXT("Enables a check-in prompt on publish game, reminding the user to check-in to avoid losing work."),
	ECVF_Default);

FSourceControlCheckInPromptModule::FSourceControlCheckInPromptModule()
{
}

void FSourceControlCheckInPromptModule::StartupModule()
{
	SourceControlCheckInPrompter = MakeShared<FSourceControlCheckInPrompter>();
	SourceControlCheckInPrompter->Init();

	ProviderChangedHandle = ISourceControlModule::Get().RegisterProviderChanged(
		FSourceControlProviderChanged::FDelegate::CreateRaw(this, &FSourceControlCheckInPromptModule::OnProviderChanged)
	);
}

void FSourceControlCheckInPromptModule::ShutdownModule()
{
	if (ProviderChangedHandle.IsValid())
	{
		ISourceControlModule::Get().UnregisterProviderChanged(ProviderChangedHandle);
		ProviderChangedHandle.Reset();
	}
	
	SourceControlCheckInPrompter.Reset();
}

void FSourceControlCheckInPromptModule::ShowModal(const FText& InMessage)
{
	FText Title = LOCTEXT("CheckInModal_Title", "Check In Changes");

	FSuppressableWarningDialog::FSetupInfo Info(InMessage, Title, "CheckInModal_Warning");
	Info.ConfirmText = LOCTEXT("CheckInModal_Yes", "Check In Changes");
	Info.CancelText = LOCTEXT("CheckInModal_No", "Skip Check-In");
	Info.CheckBoxText = FText::GetEmpty(); // Not suppressable!
	Info.Image = const_cast<FSlateBrush*>(FRevisionControlStyleManager::Get().GetBrush("RevisionControl.StatusBar.HasLocalChanges"));

	FSuppressableWarningDialog CheckInWarning(Info);
	if (CheckInWarning.ShowModal() != FSuppressableWarningDialog::Cancel)
	{
		FSourceControlWindows::ChoosePackagesToCheckIn();
	}
}

void FSourceControlCheckInPromptModule::ShowToast(const FText& InMessage)
{
	if (TSharedPtr<SNotificationItem> CheckInNotificationPin = CheckInNotification.Pin())
	{
		CheckInNotificationPin->SetCompletionState(SNotificationItem::CS_None);
		CheckInNotificationPin->ExpireAndFadeout();
	}

	FNotificationInfo Info(InMessage);
	Info.bFireAndForget = false;
	Info.bUseThrobber = false;
	Info.ExpireDuration = 0.0f;
	Info.FadeOutDuration = 0.0f;
	Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("CheckInToast_CheckIn", "Check In"), LOCTEXT("CheckInToast_CheckIn_Tooltip", "Open the submit files dialog to check in your work"), FSimpleDelegate::CreateRaw(this, &FSourceControlCheckInPromptModule::OnNotificationCheckInClicked)));
	Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("CheckInToast_Dismiss", "Dismiss"), LOCTEXT("CheckInToast_Dismiss_Tooltip", "Dismiss this notification"), FSimpleDelegate::CreateRaw(this, &FSourceControlCheckInPromptModule::OnNotificationDismissClicked)));
	Info.Image = FRevisionControlStyleManager::Get().GetBrush("RevisionControl.StatusBar.HasLocalChanges");
	CheckInNotification = FSlateNotificationManager::Get().AddNotification(Info);

	if (TSharedPtr<SNotificationItem> CheckInNotificationPin = CheckInNotification.Pin())
	{
		CheckInNotificationPin->SetCompletionState(SNotificationItem::CS_Pending);
	}
}

void FSourceControlCheckInPromptModule::OnNotificationCheckInClicked()
{
	if (TSharedPtr<SNotificationItem> CheckInNotificationPin = CheckInNotification.Pin())
	{
		CheckInNotificationPin->SetCompletionState(SNotificationItem::CS_None);
		CheckInNotificationPin->SetExpireDuration(0.0f);
		CheckInNotificationPin->SetFadeOutDuration(0.0f);
		CheckInNotificationPin->ExpireAndFadeout();
	}

	FSourceControlWindows::ChoosePackagesToCheckIn();
}

void FSourceControlCheckInPromptModule::OnNotificationDismissClicked()
{
	if (TSharedPtr<SNotificationItem> CheckInNotificationPin = CheckInNotification.Pin())
	{
		CheckInNotificationPin->SetCompletionState(SNotificationItem::CS_None);
		CheckInNotificationPin->SetExpireDuration(0.0f);
		CheckInNotificationPin->SetFadeOutDuration(0.0f);
		CheckInNotificationPin->ExpireAndFadeout();
	}
}

void FSourceControlCheckInPromptModule::OnProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider)
{
	if (TSharedPtr<SNotificationItem> CheckInNotificationPin = CheckInNotification.Pin())
	{
		CheckInNotificationPin->SetCompletionState(SNotificationItem::CS_None);
		CheckInNotificationPin->SetExpireDuration(0.0f);
		CheckInNotificationPin->SetFadeOutDuration(0.0f);
		CheckInNotificationPin->ExpireAndFadeout();
	}
}

IMPLEMENT_MODULE( FSourceControlCheckInPromptModule, SourceControlCheckInPrompt );

#undef LOCTEXT_NAMESPACE