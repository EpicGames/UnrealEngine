// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "ISourceControlCheckInPromptModule.h"

class FSourceControlCheckInPrompter;
class ISourceControlProvider;
class SNotificationItem;

class FSourceControlCheckInPromptModule : public ISourceControlCheckInPromptModule
{
public:
	FSourceControlCheckInPromptModule();

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** ISourceControlCheckInPromptModule implementation */
	virtual void ShowModal(const FText& InMessage) override;
	virtual void ShowToast(const FText& InMessage) override;

	/**
	 * Gets a reference to the source control check in prompt module instance.
	 *
	 * @return A reference to the source control check in prompt module.
	 */
	static FSourceControlCheckInPromptModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FSourceControlCheckInPromptModule>("SourceControlCheckInPrompt");
	}

	static FSourceControlCheckInPromptModule* TryGet()
	{
		return FModuleManager::GetModulePtr<FSourceControlCheckInPromptModule>("SourceControlCheckInPrompt");
	}

protected:
	void OnNotificationCheckInClicked();
	void OnNotificationDismissClicked();

	void OnProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider);

private:
	/** The prompter which triggers a periodic prompt */
	TSharedPtr<FSourceControlCheckInPrompter> SourceControlCheckInPrompter;

	/** The notification toast that a check-in is recommended */
	TWeakPtr<SNotificationItem> CheckInNotification;

	/** The delegate handle for provider changes */
	FDelegateHandle ProviderChangedHandle;
};
