// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlCheckInPrompter.h"
#include "SourceControlCheckInPromptModule.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "Containers/Ticker.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "HAL/IConsoleManager.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "SourceControlCheckInPrompter"

static const FTimespan IntervalNoCheckins = FTimespan::FromDays(1);
static const FTimespan IntervalBetweenPrompts = FTimespan::FromDays(1);
static const FTimespan IntervalBetweenGetSubmittedChangelists = FTimespan::FromMinutes(10);
static const FTimespan IntervalSessionLength = FTimespan::FromMinutes(30);

extern TAutoConsoleVariable<bool> CVarSourceControlEnablePeriodicCheckInPrompt;

static FString GetEditorMapName()
{
	if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
	{
		if (UPackage* EditorWorldPackage = EditorWorld->GetPackage())
		{
			const FString EditorWorldPackageName = EditorWorldPackage->GetName();
			if (!EditorWorldPackageName.StartsWith(TEXT("/Temp/")))
			{
				return EditorWorldPackageName;
			}
		}
	}

	return FString();
}

FSourceControlCheckInPrompter::FSourceControlCheckInPrompter()
	: PromptFlowMapName()
	, TimeCheckInPromptShown()
	, TimeGetSubmittedChangelistsExecuted()
	, bPromptDelayed(false)
{
}

FSourceControlCheckInPrompter::~FSourceControlCheckInPrompter()
{
	UPackage::PackageSavedWithContextEvent.RemoveAll(this);
}

void FSourceControlCheckInPrompter::Init()
{
	UPackage::PackageSavedWithContextEvent.AddRaw(this, &FSourceControlCheckInPrompter::OnPackageSaved);

	// Set a ticker to periodically check if there have been any project changes.
	FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateSPLambda(this,
			[this] (float DeltaTime)
			{
				FString SourceControlProjectDir = ISourceControlModule::Get().GetSourceControlProjectDir();
				if (ProjectDirectory != SourceControlProjectDir)
				{
					ProjectDirectory = SourceControlProjectDir;
					ProjectActivationTime = FDateTime::UtcNow();
					bPromptDelayed = false;
				}

				if (bPromptDelayed)
				{
					CheckPrompt();
				}

				return true;
			}
	), 60.f);
}

void FSourceControlCheckInPrompter::OnPackageSaved(const FString& Filename, UPackage* Pkg, FObjectPostSaveContext ObjectSaveContext)
{
	CheckPrompt();
}

// Step 1: Initiate the periodic prompt flow.
//         Trigger a SourceControl operation to determine how much changes the user submitted in the past day.
void FSourceControlCheckInPrompter::OnStartPrompt()
{
	// Execute it with a filter that looks for submitted changelist for the current user in the last day.
	TSharedRef<FGetSubmittedChangelists> Operation = ISourceControlOperation::Create<FGetSubmittedChangelists>();
	Operation->SetDateToFilter(FDateTime::UtcNow());
	Operation->SetDateFromFilter(FDateTime::UtcNow() - IntervalNoCheckins);
	Operation->SetOwnedFilter(true);

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	if (SourceControlProvider.IsAvailable() && SourceControlProvider.CanExecuteOperation(Operation))
	{
		// Start flow.
		PromptFlowMapName = GetEditorMapName();

		// Update the time the get submitted changelists operation was last executed.
		TimeGetSubmittedChangelistsExecuted.Add(PromptFlowMapName, FDateTime::Now());

		// Execute it asynchronously.
		SourceControlProvider.Execute(Operation, EConcurrency::Asynchronous,
			FSourceControlOperationComplete::CreateSP(this, &FSourceControlCheckInPrompter::OnSourceControlOperationComplete)
		);
	}
}

// Step 2: When the SourceControl operation completes, check if any check-ins were found.
//         If yes, end flow.
//         If no, continue flow by showing the prompt.
void FSourceControlCheckInPrompter::OnSourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	if (InOperation->GetName() == TEXT("GetSubmittedChangelists"))
	{
		if (InResult == ECommandResult::Succeeded)
		{
			TSharedRef<FGetSubmittedChangelists, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FGetSubmittedChangelists>(InOperation);

			const TArray<FSourceControlChangelistRef> SubmittedChangelists = Operation->GetSubmittedChangelists();
			if (SubmittedChangelists.Num() == 0)
			{
				// Continue flow.
				// Prompt user as he's got no recent check-ins.
				FTSTicker::GetCoreTicker().AddTicker(
					FTickerDelegate::CreateSP(this, &FSourceControlCheckInPrompter::OnAttemptPrompt), 1.0f
				);
			}
			else
			{
				PromptFlowMapName.Empty();
			}
		}
		else
		{
			// If operation failed, end flow and mark as delayed so we try again in the next cycle
			bPromptDelayed = true;
			TimeGetSubmittedChangelistsExecuted.Remove(PromptFlowMapName);
			PromptFlowMapName.Empty();
		}
	}
}

// Step 3: When it's determined that a prompt should be shown, wait for a suitable moment in the editor.
bool FSourceControlCheckInPrompter::OnAttemptPrompt(float)
{
	// Abort if map changed, project closed, etc...
	FString EditorMapName = GetEditorMapName();
	if (PromptFlowMapName != EditorMapName)
	{
		PromptFlowMapName.Empty();
		return false;
	}

	// Update the time the prompt was last shown.
	TimeCheckInPromptShown.Add(EditorMapName, FDateTime::Now());

	// Show the prompt.
	FSourceControlCheckInPromptModule::Get().ShowToast(
		LOCTEXT("MessageToast", "It's been more than 24 hours since you checked in. Check in to revision control to back up your project?")
	);

	PromptFlowMapName.Empty();
	return false;
}

void FSourceControlCheckInPrompter::CheckPrompt()
{
	FString EditorMapName = GetEditorMapName();
	if (PromptFlowMapName.IsEmpty() && !EditorMapName.IsEmpty())
	{
		bool bPromptEnabled = CVarSourceControlEnablePeriodicCheckInPrompt.GetValueOnGameThread();
		if (!bPromptEnabled)
		{
			return;
		}

		// If the user has not been active in this world for a sufficient amount of time,
		// the prompt is delayed until the required interval has passed.
		bPromptDelayed = (FDateTime::UtcNow() - IntervalSessionLength < ProjectActivationTime);

		if (bPromptDelayed)
		{
			return;
		}

		// Otherwise, validate we can get latest changes and we actually have local changes to check in
		bool bIsPromptAllowed = IsPromptAllowed();
		bool bIsGetSubmittedChangelistsAllowed = IsGetSubmittedChangelistsAllowed();
		bool bPromptShow = bIsPromptAllowed && bIsGetSubmittedChangelistsAllowed;
		if (bPromptShow)
		{
			OnStartPrompt();
		}
	}
}

bool FSourceControlCheckInPrompter::IsPromptAllowed() const
{
	// Ensure the SourceControl system is available.
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	if (!SourceControlProvider.IsAvailable())
	{
		return false;
	}

	// Ensure it has something to check-in.
	if (SourceControlProvider.HasChangesToCheckIn().IsSet())
	{
		if (!SourceControlProvider.HasChangesToCheckIn().GetValue())
		{
			return false;
		}
	}

	// Ensure there's a world in the editor.
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	if (EditorWorld == nullptr)
	{
		return false;
	}

	// Ensure the project activation time applies to the current project.
	FString SourceControlProjectDir = ISourceControlModule::Get().GetSourceControlProjectDir();
	if (ProjectDirectory != SourceControlProjectDir)
	{
		return false;
	}

	// Ensure the prompt hasn't been shown too recently for that world.
	FString PackageName = EditorWorld->GetPackage()->GetName();
	if (const FDateTime* LastPrompt = TimeCheckInPromptShown.Find(PackageName))
	{
		return FDateTime::Now() >= *LastPrompt + IntervalBetweenPrompts;
	}
	else
	{
		return true; // A prompt wasn't shown before.
	}
}

bool FSourceControlCheckInPrompter::IsGetSubmittedChangelistsAllowed() const
{
	// Ensure the SourceControl system is available.
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	if (!SourceControlProvider.IsAvailable())
	{
		return false;
	}

	// Ensure there's a world in the editor.
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	if (EditorWorld == nullptr)
	{
		return false;
	}

	// Ensure the operation isn't executed too often for that world.
	FString PackageName = EditorWorld->GetPackage()->GetName();
	if (const FDateTime* LastOperation = TimeGetSubmittedChangelistsExecuted.Find(PackageName))
	{
		return FDateTime::Now() >= *LastOperation + IntervalBetweenGetSubmittedChangelists;
	}
	else
	{
		return true; // An operation wasn't executed yet.
	}
}

#undef LOCTEXT_NAMESPACE