// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "ISourceControlProvider.h"

class FObjectPostSaveContext;

class FSourceControlCheckInPrompter : public TSharedFromThis<FSourceControlCheckInPrompter>
{
public:
	FSourceControlCheckInPrompter();
	virtual ~FSourceControlCheckInPrompter();

	/**
	 * Initializes the instance.
	 */
	void Init();

private:
	/**
	 * Called when a package has been saved.
	 *
	 * @param Filename The filename the package was saved to
	 * @param Obj The package that was saved
	 */
	void OnPackageSaved(const FString& Filename, UPackage* Pkg, FObjectPostSaveContext ObjectSaveContext);

	/**
	 * Called when a prompt flow should start, which might result in one being shown if
	 * it's been over a day since the last commit.
	 */
	void OnStartPrompt();

	/**
	 * Called when an executed source control operation completes.
	 */
	void OnSourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);

	/**
	 * Called when a check-in prompt should attempted to be shown.
	 * @param DeltaTime argument is the time since the last game frame
	 */
	bool OnAttemptPrompt(float);

	/**
	 * Checks if it's allowed to display the periodic check-in prompt.
	 * The prompt is only expected to be shown once every X hours.
	 */
	bool IsPromptAllowed() const;

	/**
	 * Checks if it's allow to perform an asynchronous get submitted changelist operation.
	 * The operation is only expected to be executed once every X minutes.
	 */
	bool IsGetSubmittedChangelistsAllowed() const;

	/**
	 * Called when a check should happen whether a prompt should be shown.
	 */
	void CheckPrompt();

private:
	/** When non empty, contains the package name of the map the prompt will be shown for */
	FString PromptFlowMapName;

	/** Time the last prompt was shown by map name */
	TMap<FString, FDateTime> TimeCheckInPromptShown;

	/** Time the last get submitted changelists operation was performed by map name */
	TMap<FString, FDateTime> TimeGetSubmittedChangelistsExecuted;

	/** The project directory user is working in */
	FString ProjectDirectory;

	/** The project activation time (eg: when was it opened) */
	FDateTime ProjectActivationTime;

	/** The prompt has been delayed */
	bool bPromptDelayed;

};