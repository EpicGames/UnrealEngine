// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Delegates/Delegate.h"

class ISourceControlCheckInPromptModule : public IModuleInterface
{
public:
	/**
	 * Shows a check-in prompt reminding the user that it's important to frequently save their work.
	 * @param InMessage		The message to display in the message box
	 */
	virtual void ShowModal(const FText& InMessage) = 0;

	/**
	 * Shows a check-in toast reminder the user that it's important to frequently save their work.
	 * @param InMessage		The message to display in the toast
	 */
	virtual void ShowToast(const FText& InMessage) = 0;

public:
	/**
	 * Gets a reference to the source control check in prompt module instance.
	 *
	 * @return A reference to the source control check in module.
	 */
	static inline ISourceControlCheckInPromptModule& Get()
	{
		static FName SourceControlCheckInPromptModule("SourceControlCheckInPrompt");
		return FModuleManager::LoadModuleChecked<ISourceControlCheckInPromptModule>(SourceControlCheckInPromptModule);
	}

	static inline ISourceControlCheckInPromptModule* TryGet()
	{
		static FName SourceControlCheckInPromptModule("SourceControlCheckInPrompt");
		return FModuleManager::GetModulePtr<ISourceControlCheckInPromptModule>(SourceControlCheckInPromptModule);
	}
};
