// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SourceControlCheckInPrompt : ModuleRules
{
	public SourceControlCheckInPrompt(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"UnrealEd",
				"Slate",
				"SlateCore",
				"SourceControl",
				"SourceControlWindows"
			}
		);
	}
}
